// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class FileLogging : ModuleRules
	{
		public FileLogging( ReadOnlyTargetRules Target ) : base(Target)
		{
            PublicDependencyModuleNames.AddRange(
                new string[]
				{
					"Core",
					"CoreUObject",
                    "Engine",
					// ... add other public dependencies that you statically link with here ...
				}
                );

            PrivateDependencyModuleNames.AddRange(
                new string[]
				{
					"Analytics",
					// ... add private dependencies that you statically link with here ...
				}
                );

            PublicIncludePathModuleNames.Add("Analytics");
        }
	}
}
