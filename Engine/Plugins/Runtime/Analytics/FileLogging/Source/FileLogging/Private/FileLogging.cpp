// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileLogging.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "FileLoggingProvider.h"

DEFINE_LOG_CATEGORY_STATIC(LogFileLoggingAnalytics, Display, All);

IMPLEMENT_MODULE( FAnalyticsFileLogging, FileLogging )

void FAnalyticsFileLogging::StartupModule()
{
	FileLoggingProvider = MakeShareable(new FAnalyticsProviderFileLogging());
}

void FAnalyticsFileLogging::ShutdownModule()
{
	if (FileLoggingProvider.IsValid())
	{
		FileLoggingProvider->EndSession();
	}
}

TSharedPtr<IAnalyticsProvider> FAnalyticsFileLogging::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	return FileLoggingProvider;
}

// Provider

FAnalyticsProviderFileLogging::FAnalyticsProviderFileLogging() :
	bHasSessionStarted(false),
	bHasWrittenFirstEvent(false),
	Age(0),
	FileArchive(nullptr)
{
	FileArchive = nullptr;
	AnalyticsFilePath = FPaths::ProjectSavedDir() + TEXT("Analytics/");
	UserId = FPlatformMisc::GetLoginId();
}

FAnalyticsProviderFileLogging::~FAnalyticsProviderFileLogging()
{
	if (bHasSessionStarted)
	{
		EndSession();
	}
}

bool FAnalyticsProviderFileLogging::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		EndSession();
	}
	SessionId = UserId + TEXT("-") + FDateTime::Now().ToString();
	const FString FileName = AnalyticsFilePath + SessionId + TEXT(".analytics");
	// Close the old file and open a new one
	FileArchive = IFileManager::Get().CreateFileWriter(*FileName);
	if (FileArchive != nullptr)
	{
		FileArchive->Logf(TEXT("{"));
		FileArchive->Logf(TEXT("\t\"sessionId\" : \"%s\","), *SessionId);
		FileArchive->Logf(TEXT("\t\"userId\" : \"%s\","), *UserId);
		if (BuildInfo.Len() > 0)
		{
			FileArchive->Logf(TEXT("\t\"buildInfo\" : \"%s\","), *BuildInfo);
		}
		if (Age != 0)
		{
			FileArchive->Logf(TEXT("\t\"age\" : %d,"), Age);
		}
		if (Gender.Len() > 0)
		{
			FileArchive->Logf(TEXT("\t\"gender\" : \"%s\","), *Gender);
		}
		if (Location.Len() > 0)
		{
			FileArchive->Logf(TEXT("\t\"location\" : \"%s\","), *Location);
		}
		FileArchive->Logf(TEXT("\t\"events\" : ["));
		bHasSessionStarted = true;
		UE_LOGF(LogFileLoggingAnalytics, Display, "Session created file (%ls) for user (%ls)", *FileName, *UserId);
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::StartSession failed to create file to log analytics events to");
	}
	return bHasSessionStarted;
}

void FAnalyticsProviderFileLogging::EndSession()
{
	if (FileArchive != nullptr)
	{
		FileArchive->Logf(TEXT("\t]"));
		FileArchive->Logf(TEXT("}"));
		FileArchive->Flush();
		FileArchive->Close();
		delete FileArchive;
		FileArchive = nullptr;
		UE_LOGF(LogFileLoggingAnalytics, Display, "Session ended for user (%ls) and session id (%ls)", *UserId, *SessionId);
	}
	bHasWrittenFirstEvent = false;
	bHasSessionStarted = false;
}

void FAnalyticsProviderFileLogging::FlushEvents()
{
	if (FileArchive != nullptr)
	{
		FileArchive->Flush();
		UE_LOGF(LogFileLoggingAnalytics, Display, "Analytics file flushed");
	}
}

void FAnalyticsProviderFileLogging::SetUserID(const FString& InUserID)
{
	if (!bHasSessionStarted)
	{
		UserId = InUserID;
		UE_LOGF(LogFileLoggingAnalytics, Display, "User is now (%ls)", *UserId);
	}
	else
	{
		// Log that we shouldn't switch users during a session
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::SetUserID called while a session is in progress. Ignoring.");
	}
}

FString FAnalyticsProviderFileLogging::GetUserID() const
{
	return UserId;
}

FString FAnalyticsProviderFileLogging::GetSessionID() const
{
	return SessionId;
}

bool FAnalyticsProviderFileLogging::SetSessionID(const FString& InSessionID)
{
	if (!bHasSessionStarted)
	{
		SessionId = InSessionID;
		UE_LOGF(LogFileLoggingAnalytics, Display, "Session is now (%ls)", *SessionId);
	}
	else
	{
		// Log that we shouldn't switch session ids during a session
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::SetSessionID called while a session is in progress. Ignoring.");
	}
	return !bHasSessionStarted;
}

void FAnalyticsProviderFileLogging::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT(","));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventName\" : \"%s\""), *EventName);
		if (Attributes.Num() > 0)
		{
			FileArchive->Logf(TEXT(",\t\t\t\"attributes\" : ["));
			bool bHasWrittenFirstAttr = false;
			// Write out the list of attributes as an array of attribute objects
			for (auto Attr : DefaultEventAttributes)
			{
				if (bHasWrittenFirstAttr)
				{
					FileArchive->Logf(TEXT("\t\t\t,"));
				}
				FileArchive->Logf(TEXT("\t\t\t{"));
				FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
				if (Attr.IsJsonFragment())
				{
					FileArchive->Logf(TEXT("\t\t\t\t\"value\" : %s"), *Attr.GetValue());
				}
				else
				{
					FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
				}
				FileArchive->Logf(TEXT("\t\t\t}"));
				bHasWrittenFirstAttr = true;
			}

			for (auto Attr : Attributes)
			{
				if (bHasWrittenFirstAttr)
				{
					FileArchive->Logf(TEXT("\t\t\t,"));
				}
				FileArchive->Logf(TEXT("\t\t\t{"));
				FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
				if (Attr.IsJsonFragment())
				{
					FileArchive->Logf(TEXT("\t\t\t\t\"value\" : %s"), *Attr.GetValue());
				}
				else
				{
					FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
				}
				FileArchive->Logf(TEXT("\t\t\t}"));
				bHasWrittenFirstAttr = true;
			}
			FileArchive->Logf(TEXT("\t\t\t]"));
		}
		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Analytics event (%ls) written with (%d) attributes", *EventName, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordEvent called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventName\" : \"recordItemPurchase\","));

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));

		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"itemId\", \t\"value\" : \"%s\" },"), *ItemId);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"currency\", \t\"value\" : \"%s\" },"), *Currency);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"perItemCost\", \t\"value\" : \"%d\" },"), PerItemCost);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"itemQuantity\", \t\"value\" : \"%d\" }"), ItemQuantity);

		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "(%d) number of item (%ls) purchased with (%ls) at a cost of (%d) each", ItemQuantity, *ItemId, *Currency, PerItemCost);
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordItemPurchase called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventName\" : \"recordCurrencyPurchase\","));

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));

		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"gameCurrencyType\", \t\"value\" : \"%s\" },"), *GameCurrencyType);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"gameCurrencyAmount\", \t\"value\" : \"%d\" },"), GameCurrencyAmount);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"realCurrencyType\", \t\"value\" : \"%s\" },"), *RealCurrencyType);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"realMoneyCost\", \t\"value\" : \"%f\" },"), RealMoneyCost);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"paymentProvider\", \t\"value\" : \"%s\" }"), *PaymentProvider);

		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "(%d) amount of in game currency (%ls) purchased with (%ls) at a cost of (%f) each", GameCurrencyAmount, *GameCurrencyType, *RealCurrencyType, RealMoneyCost);
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordCurrencyPurchase called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventName\" : \"recordCurrencyGiven\","));

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));

		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"gameCurrencyType\", \t\"value\" : \"%s\" },"), *GameCurrencyType);
		FileArchive->Logf(TEXT("\t\t\t\t{ \"name\" : \"gameCurrencyAmount\", \t\"value\" : \"%d\" }"), GameCurrencyAmount);

		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "(%d) amount of in game currency (%ls) given to user", GameCurrencyAmount, *GameCurrencyType);
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordCurrencyGiven called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::SetAge(int InAge)
{
	Age = InAge;
}

void FAnalyticsProviderFileLogging::SetLocation(const FString& InLocation)
{
	Location = InLocation;
}

void FAnalyticsProviderFileLogging::SetGender(const FString& InGender)
{
	Gender = InGender;
}

void FAnalyticsProviderFileLogging::SetBuildInfo(const FString& InBuildInfo)
{
	BuildInfo = InBuildInfo;
}

void FAnalyticsProviderFileLogging::RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"error\" : \"%s\","), *Error);

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));
		bool bHasWrittenFirstAttr = false;
		// Write out the list of attributes as an array of attribute objects
		for (auto Attr : Attributes)
		{
			if (bHasWrittenFirstAttr)
			{
				FileArchive->Logf(TEXT("\t\t\t,"));
			}
			FileArchive->Logf(TEXT("\t\t\t{"));
			FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
			FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
			FileArchive->Logf(TEXT("\t\t\t}"));
			bHasWrittenFirstAttr = true;
		}
		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Error is (%ls) number of attributes is (%d)", *Error, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordError called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordProgress(const FString& ProgressType, const FString& ProgressName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventType\" : \"Progress\","));
		FileArchive->Logf(TEXT("\t\t\t\"progressType\" : \"%s\","), *ProgressType);
		FileArchive->Logf(TEXT("\t\t\t\"progressName\" : \"%s\","), *ProgressName);

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));
		bool bHasWrittenFirstAttr = false;
		// Write out the list of attributes as an array of attribute objects
		for (auto Attr : Attributes)
		{
			if (bHasWrittenFirstAttr)
			{
				FileArchive->Logf(TEXT("\t\t\t,"));
			}
			FileArchive->Logf(TEXT("\t\t\t{"));
			FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
			FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
			FileArchive->Logf(TEXT("\t\t\t}"));
			bHasWrittenFirstAttr = true;
		}
		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Progress event is type (%ls), named (%ls), number of attributes is (%d)", *ProgressType, *ProgressName, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordProgress called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordItemPurchase(const FString& ItemId, int ItemQuantity, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventType\" : \"ItemPurchase\","));
		FileArchive->Logf(TEXT("\t\t\t\"itemId\" : \"%s\","), *ItemId);
		FileArchive->Logf(TEXT("\t\t\t\"itemQuantity\" : %d,"), ItemQuantity);

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));
		bool bHasWrittenFirstAttr = false;
		// Write out the list of attributes as an array of attribute objects
		for (auto Attr : Attributes)
		{
			if (bHasWrittenFirstAttr)
			{
				FileArchive->Logf(TEXT("\t\t\t,"));
			}
			FileArchive->Logf(TEXT("\t\t\t{"));
			FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
			FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
			FileArchive->Logf(TEXT("\t\t\t}"));
			bHasWrittenFirstAttr = true;
		}
		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Item purchase id (%ls), quantity (%d), number of attributes is (%d)", *ItemId, ItemQuantity, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordItemPurchase called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventType\" : \"CurrencyPurchase\","));
		FileArchive->Logf(TEXT("\t\t\t\"gameCurrencyType\" : \"%s\","), *GameCurrencyType);
		FileArchive->Logf(TEXT("\t\t\t\"gameCurrencyAmount\" : %d,"), GameCurrencyAmount);

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));
		bool bHasWrittenFirstAttr = false;
		// Write out the list of attributes as an array of attribute objects
		for (auto Attr : Attributes)
		{
			if (bHasWrittenFirstAttr)
			{
				FileArchive->Logf(TEXT("\t\t\t,"));
			}
			FileArchive->Logf(TEXT("\t\t\t{"));
			FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
			FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
			FileArchive->Logf(TEXT("\t\t\t}"));
			bHasWrittenFirstAttr = true;
		}
		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Currency purchase type (%ls), quantity (%d), number of attributes is (%d)", *GameCurrencyType, GameCurrencyAmount, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordCurrencyPurchase called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	if (bHasSessionStarted)
	{
		check(FileArchive != nullptr);

		if (bHasWrittenFirstEvent)
		{
			FileArchive->Logf(TEXT("\t\t,"));
		}
		bHasWrittenFirstEvent = true;

		FileArchive->Logf(TEXT("\t\t{"));
		FileArchive->Logf(TEXT("\t\t\t\"eventType\" : \"CurrencyGiven\","));
		FileArchive->Logf(TEXT("\t\t\t\"gameCurrencyType\" : \"%s\","), *GameCurrencyType);
		FileArchive->Logf(TEXT("\t\t\t\"gameCurrencyAmount\" : %d,"), GameCurrencyAmount);

		FileArchive->Logf(TEXT("\t\t\t\"attributes\" :"));
		FileArchive->Logf(TEXT("\t\t\t["));
		bool bHasWrittenFirstAttr = false;
		// Write out the list of attributes as an array of attribute objects
		for (auto Attr : Attributes)
		{
			if (bHasWrittenFirstAttr)
			{
				FileArchive->Logf(TEXT("\t\t\t,"));
			}
			FileArchive->Logf(TEXT("\t\t\t{"));
			FileArchive->Logf(TEXT("\t\t\t\t\"name\" : \"%s\","), *Attr.GetName());
			FileArchive->Logf(TEXT("\t\t\t\t\"value\" : \"%s\""), *Attr.GetValue());
			FileArchive->Logf(TEXT("\t\t\t}"));
			bHasWrittenFirstAttr = true;
		}
		FileArchive->Logf(TEXT("\t\t\t]"));

		FileArchive->Logf(TEXT("\t\t}"));

		UE_LOGF(LogFileLoggingAnalytics, Display, "Currency given type (%ls), quantity (%d), number of attributes is (%d)", *GameCurrencyType, GameCurrencyAmount, Attributes.Num());
	}
	else
	{
		UE_LOGF(LogFileLoggingAnalytics, Warning, "FAnalyticsProviderFileLogging::RecordCurrencyGiven called before StartSession. Ignoring.");
	}
}

void FAnalyticsProviderFileLogging::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderFileLogging::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderFileLogging::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderFileLogging::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes[AttributeIndex];
}