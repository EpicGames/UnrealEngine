// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProvider.h"

struct FAnalyticsEventAttribute;

class Error;

class FAnalyticsProviderFileLogging :
	public IAnalyticsProvider
{
	/** Path where analytics files are saved out */
	FString AnalyticsFilePath;
	/** Tracks whether we need to start the session or restart it */
	bool bHasSessionStarted;
	/** Whether an event was written before or not */
	bool bHasWrittenFirstEvent;
	/** Id representing the user the analytics are recording for */
	FString UserId;
	/** Unique Id representing the session the analytics are recording for */
	FString SessionId;
	/** Holds the Age if set */
	int32 Age;
	/** Holds the Location of the user if set */
	FString Location;
	/** Holds the Gender of the user if set */
	FString Gender;
	/** Holds the build info if set */
	FString BuildInfo;
	/** The file archive used to write the data */
	FArchive* FileArchive;

public:
	FAnalyticsProviderFileLogging();
	virtual ~FAnalyticsProviderFileLogging();

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void EndSession() override;
	virtual void FlushEvents() override;

	virtual void SetUserID(const FString& InUserID) override;
	virtual FString GetUserID() const override;

	virtual FString GetSessionID() const override;
	virtual bool SetSessionID(const FString& InSessionID) override;

	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;

	virtual void RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity) override;

	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider) override;

	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount) override;

	virtual void SetBuildInfo(const FString& InBuildInfo) override;
	virtual void SetGender(const FString& InGender) override;
	virtual void SetLocation(const FString& InLocation) override;
	virtual void SetAge(const int32 InAge) override;

	virtual void RecordItemPurchase(const FString& ItemId, int ItemQuantity, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;

private:
	TArray<FAnalyticsEventAttribute> DefaultEventAttributes;
};
