// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"

class IAnalyticsProvider;

/**
 * The public interface to this module
 */
class FAnalyticsFileLogging :
	public IAnalyticsProviderModule
{
	/** Singleton for analytics */
	TSharedPtr<IAnalyticsProvider> FileLoggingProvider;

	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsFileLogging& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsFileLogging >( "FileLogging" );
	}

	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;
	
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

